[/Script/Engine.Engine]
; DefaultBloomKernel is not used on mobile
DefaultBloomKernelTextureName=/Engine/EngineResources/DefaultTexture.DefaultTexture

; Default film grain is not used on mobile
DefaultFilmGrainTextureName=""

[/Script/Engine.GarbageCollectionSettings]
gc.MaxObjectsInGame=131072

[/Script/Engine.RendererSettings]
r.DistanceFields=0
r.TemporalAA.Mobile.UseCompute=0
; Disables parallel GDME by default due to not enough threads
r.Visibility.DynamicMeshElements.Parallel=0
; Volumetric Clouds are not supported by mobile renderer, skip materials/global shaders compilation.
r.VolumetricCloud.Support=0
r.OIT.SortedTriangles=0

[/Script/UdpMessaging.UdpMessagingSettings]
; IOS requires an additional entitlement (com.apple.developer.networking.multicast) to enable UDP Local Networking
EnableTransport=False

[Advertising]
DefaultProviderName=IOSAdvertising

[AlternateTextureCompression]
;disabling TextureCompressionFormat for Android/IOS no longer needed
;TextureCompressionFormat=""
;TextureFormatPrefix=""

[Audio]
AudioCaptureModuleName=AudioCaptureAudioUnit
; Defining below allows switching to audio mixer using -audiomixer commandline
AudioMixerModuleName=AudioMixerAudioUnit

; Defines a platform-specific volume headroom (in dB) for audio to provide better platform consistency with respect to volume levels.
PlatformHeadroomDB=-6

PlatformFormat=ADPCM
PlatformStreamingFormat=ADPCM

[BackgroundHttp.iOSSettings]
; Use foreground-only NSURLSession 
bUseForegroundSession=false
; Requests will be scheduled on discretion of iOS if true, otherwise they will be scheduled ASAP if false
bDiscretionary=false
; Wakeup app in background and report state of current downloads, allows to retry downloads in background
bShouldSendLaunchEvents=true
; Maximum connenctions per host for whole session
MaximumConnectionsPerHost=6
; Time out and retry if we don't get a response in this timeframe if app is in foreground and bUseForegroundSession is true
BackgroundReceiveTimeout=120
; Time out and retry if the download hasn't finished in this long, regardless of app state 
BackgroundHttpResourceTimeout=3600
; How many times we should try using same CDN before moving onto a different CDN ( -1 = unlimited, 0 = only try first CDN, 1 = try each CDN once )
RetryResumeDataLimit=3
; How many milliseconds to wait for HEAD replies on first CDN request to decide if CDN is reachable or not (0 = disabled).
; Unreachable CDNs will be deprioritized in all download requests.
CDNReorderingTimeout=400
; Reoder by ping time CDNs that responded with-in CDNReorderingTimeout. No-op if CDNReorderingTimeout is disabled.
bCDNReorderByPingTime=false
; When in foreground every N seconds check if downloads are not receiving bytes (regardless of bUseForegroundSession), retry them if so.
CheckForForegroundStaleDownloadsWithInterval=1
; How long to wait in seconds before considering retrying download in CheckForForegroundStaleDownloadsWithInterval
ForegroundStaleDownloadTimeout=60

[ConsoleVariables]
Input.EnableNativeGestureRecognition=true
Slate.CacheRenderData=0
r.HairStrands.Simulation=0
r.HairStrands.Strands=0
r.HairStrands.Binding=0
r.Forward.LightGridSizeZ=8
r.Forward.MaxCulledLightsPerCell=8

; enable stripping debug strings from streamable manager in shipping to save memory
s.StreamableStripDebugName=true
s.StreamableStripDebugNameInShipping=true

[DeviceProfileManager]
DeviceProfileSelectionModule="IOSDeviceProfileSelector"
PreviewDeviceProfileSelectionModule="IOSPreviewDeviceProfileSelector"

[LocalNotification]
DefaultPlatformService=IOSLocalNotification

[MemoryMappedFiles]
Enable=true
Alignment=1

[OnlineSubsystem]
DefaultPlatformService=IOS
NativePlatformService=IOS
LocalPlatformName=IOS

[OnlineSubsystemIOS.Store]
bSupportsInAppPurchasing=true

[PlatformCrypto]
PlatformRequiresDataCrypto=True
PakSigningRequired=False

[PlatformMemoryBuckets]
;iOS specific: uses available memory on app start instead of total device memory
LargestMemoryBucket_MinGB=6
LargerMemoryBucket_MinGB=4
DefaultMemoryBucket_MinGB=4
SmallerMemoryBucket_MinGB=3
SmallestMemoryBucket_MinGB=2

[Ping]
StackSize=0

[SlateRenderer]
NumPreallocatedVertices=200

[SystemSettings]
; iOS should always have IPv6 support on
net.DisableIPv6=0
;Perplatform to PerQualityLevel conversion mapping for platform groups
QualityLevelMapping="low"

;Manual range settings for Niagara quality levels. Simplifies handling of device profiles/fragments and what quality levels they can set.
;Please ensure device profiles or fragments for this platform do not set fx.Niagara.QualityLevel values outside of this range
fx.Niagara.QualityLevel.Min=0
fx.Niagara.QualityLevel.Max=1

[TextureStreaming]
; PoolSizeVRAMPercentage is how much percentage of GPU Dedicated VRAM should be used as a TexturePool cache for streaming textures (0 - unlimited streaming)
PoolSizeVRAMPercentage=70

[BuildPatchTool]
VerificationThreadCount=3

[ShaderPlatformConfig METAL_ES3_1_IOS]
BindlessConfiguration=Disabled
	
[ShaderPlatformConfig METAL_SM5_IOS]
BindlessConfiguration=Disabled

[ShaderPlatformConfig METAL_SM6_IOS]
BindlessConfiguration=All

