[InputPlatformSettings_IOS InputPlatformSettings]
MaxPlatformUserCount=4
input.DeviceMappingPolicy=0