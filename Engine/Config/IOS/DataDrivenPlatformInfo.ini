[DataDrivenPlatformInfo]
bIsConfidential=false
TargetSettingsIniSectionName=/Script/IOSRuntimeSettings.IOSRuntimeSettings
bHasDedicatedGamepad=false
bInputSupportConfigurable=false
DefaultInputType=Touch
bSupportsMouseAndKeyboard=false
bSupportsGamepad=true
bCanChangeGamepadType=true
bSupportsTouch=true
GlobalIdentifier=988EBA73F971495BAFB09639F8C796BD

NormalIconPath=Launcher/IOS/Platform_IOS_24x
LargeIconPath=Launcher/IOS/Platform_IOS_128x
XLargeIconPath=
AutoSDKPath=
Windows:TutorialPath=SharingAndReleasing/Mobile/iOS
Mac:TutorialPath=SharingAndReleasing/Mobile/iOS
Linux:TutorialPath=SharingAndReleasing/Mobile/iOS
Windows:bIsEnabled=true
Mac:bIsEnabled=true
Linux:bIsEnabled=false
bUsesHostCompiler=false
bUATClosesAfterLaunch=true
PlatformGroupName=Mobile
ICUDataVersion=icudt64l

[PreviewPlatform IOSMetal]
PlatformName=IOS
ShaderFormat=SF_METAL_ES3_1_IOS
ShaderPlatform=METAL_ES3_1_IOS
ActiveIconName=LevelEditor.PreviewMode.iOS.Enabled
InactiveIconName=LevelEditor.PreviewMode.iOS.Disabled
MenuTooltip=NSLOCTEXT("PreviewPlatform", "PreviewMenuTooltip_iOS", "Mobile preview using iOS material quality settings.")
IconText=NSLOCTEXT("PreviewPlatform", "PreviewIconText_iOS", "iOS")
DeviceProfileName=IOS:IOS_Low:IOS_Mid:IOS_High
FriendlyName=NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_GenericIOS", "Generic IOS"):NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_IOSLow","IOS Low"):NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_IOSMid","IOS Mid"):NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_IOSHigh","IOS High")

[PreviewPlatform IOSMetalSM5]
PlatformName=IOS
ShaderFormat=SF_METAL_SM5_IOS
ShaderPlatform=METAL_SM5_IOS
EnabledCVar=ini:Engine:IOS:/Script/IOSRuntimeSettings.IOSRuntimeSettings:bSupportsMetalMobileSM5
ActiveIconName=LevelEditor.PreviewMode.iOSSM5.Enabled
InactiveIconName=LevelEditor.PreviewMode.iOSSM5.Disabled
MenuTooltip=NSLOCTEXT("PreviewPlatform", "PreviewMenuTooltip_iOSSM5", "Mobile preview using iOS SM5 material quality settings.")
IconText=NSLOCTEXT("PreviewPlatform", "PreviewIconText_iOSSM5", "iOS SM5")
DeviceProfileName=IOS

[ShaderPlatform METAL_ES3_1_IOS]
Language=Metal
MaxFeatureLevel=ES3_1
ShaderFormat=SF_METAL_ES3_1_IOS
bIsMobile=true
bTargetsTiledGPU=true
bSupports4ComponentUAVReadWrite=true
bSupportsManualVertexFetch=false
bSupportsVolumeTextureAtomics=false
bSupportsShaderPipelines = false
bSupportsIndependentSamplers=true
bSupportsComputeFramework = true
bSupportsDistanceFields=true
bSupportsBindless=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_IOSMobile", "IOS Metal Mobile")

[ShaderPlatform METAL_SM5_IOS]
Language=Metal
MaxFeatureLevel=SM5
ShaderFormat=SF_METAL_SM5_IOS
bTargetsTiledGPU=true
bSupportsIndexBufferUAVs=true
bSupports4ComponentUAVReadWrite=true
bSupportsManualVertexFetch=false
bSupportsVolumeTextureAtomics=false
bSupportsDistanceFields=true
bSupportsIndependentSamplers=true
bSupportsComputeFramework = true
bSupportsBindless=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_IOSSM5", "IOS Metal SM5")
	
[ShaderPlatform METAL_SM6_IOS]
Language=Metal
MaxFeatureLevel=SM6
ShaderFormat=SF_METAL_SM6_IOS
bTargetsTiledGPU=false
bSupportsAnisotropicMaterials=true
bSupportsDistanceFields=true
bSupportsGen5TemporalAA=true
bSupportsLumenGI=true
bSupportsSSDIndirect=true
bSupportsWaveOperations=RuntimeGuaranteed
MinimumWaveSize=32
MaximumWaveSize=64
bSupportsGPUScene=true
bSupportsFFTBloom=true
bSupportsDiaphragmDOF=true
bSupportsVertexShaderLayer=true
bSupportsDxc=true
bIsSPIRV=true
bSupportsInstancedStereo=false
SupportsMultiViewport=RuntimeGuaranteed
bSupportsPercentageCloserShadows=true
bSupports4ComponentUAVReadWrite=true
bSupportsVolumeTextureAtomics=true
bSupportsDualSourceBlending=true
EnablesHLSL2021ByDefault=1
bSupportsRayTracing=true
bSupportsRayTracingShaders=true
bSupportsInlineRayTracing=true
bSupportsRayTracingProceduralPrimitive=true
bSupportsRayTracingIndirectInstanceData=true
StaticShaderBindingLayoutSupport=RayTracingOnly
bSupportsPathTracing=true
bSupportsRayTracingCallableShaders=true
bSupportsHairStrandGeometry=true
bSupportsNanite=true
bSupportsUInt64ImageAtomics=true
bSupportsNNEShaders=true
bSupportsIndependentSamplers=true
bSupportsComputeFramework=true
bSupportsBindless=true
bSupportsMeshShadersTier0=false
bSupportsMeshShadersTier1=false
bSupportsMeshShadersWithClipDistance=false
bSupportsAdaptiveGBuffer=true
MaxMeshShaderThreadGroupSize=128
SupportsBarycentricsSemantic=RuntimeDependent
bSupportsRealTypes=RuntimeGuaranteed
bSupportsSceneDataCompressedTransforms=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_IOSSM6", "IOS Metal SM6")
	
[ShaderPlatform METAL_SIM]
Language=Metal
MaxFeatureLevel=ES3_1
ShaderFormat=SF_METAL_SIM
bIsMobile=true
bTargetsTiledGPU=true
bSupports4ComponentUAVReadWrite=true
bSupportsManualVertexFetch=false
bSupportsVolumeTextureAtomics=false
bSupportsShaderPipelines = false
bSupportsIndependentSamplers=true
bSupportsComputeFramework = true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_IOSMobileSIM", "IOS Metal Simulator Mobile")
