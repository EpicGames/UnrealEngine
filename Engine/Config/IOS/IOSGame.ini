[Staging]

[/Script/UnrealEd.ProjectPackagingSettings]
+FileExtensionsToSkipCompressionInPak=".metalmap"
+FileExtensionsToSkipCompressionInPak=".metallib"
+FileExtensionsToSkipEncryptionInPak=".metalmap"
+FileExtensionsToSkipEncryptionInPak=".metallib"