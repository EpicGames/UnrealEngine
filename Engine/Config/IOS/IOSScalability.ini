; ------------------------------------------------------------------------------
; Foliage settings
; ------------------------------------------------------------------------------

[FoliageQuality@0]
foliage.DensityScale=0.5
grass.DensityScale=0.5

[FoliageQuality@1]
grass.densityScale=0.8
grass.CullDistanceScale=0.8
foliage.DensityScale=0.8

[FoliageQuality@2]
grass.densityScale=1.0
grass.CullDistanceScale=1.0
foliage.DensityScale=1.0

[FoliageQuality@3]
grass.densityScale=1.0
grass.CullDistanceScale=1.0
foliage.DensityScale=1.0

; ------------------------------------------------------------------------------
; View distance settings
; ------------------------------------------------------------------------------

[ViewDistanceQuality@0]
r.ViewDistanceScale=0.70
r.NeverOcclusionTestDistance=200

[ViewDistanceQuality@1]
r.ViewDistanceScale=1.0
r.NeverOcclusionTestDistance=200

[ViewDistanceQuality@2]
r.ViewDistanceScale=1.0
r.NeverOcclusionTestDistance=200

[ViewDistanceQuality@3]
r.ViewDistanceScale=1.0
r.NeverOcclusionTestDistance=200

; ------------------------------------------------------------------------------
; Shadow settings
; ------------------------------------------------------------------------------

[ShadowQuality@0]
r.ShadowQuality=0
r.Shadow.MaxCSMResolution=1024
r.Shadow.DistanceScale=0.5
r.Shadow.CSM.MaxCascades=1
r.Shadow.RadiusThreshold=0.06
r.VolumetricFog=0
r.ContactShadows=0
r.DistanceFieldShadowing=0
r.DFShadowQuality=0

[ShadowQuality@1]
r.ShadowQuality=1
r.Shadow.MaxCSMResolution=1024
r.Shadow.DistanceScale=0.7
r.Shadow.CSM.MaxCascades=1
r.Shadow.RadiusThreshold=0.06
r.VolumetricFog=0
r.ContactShadows=0
r.DistanceFieldShadowing=0
r.DFShadowQuality=0

[ShadowQuality@2]
r.ShadowQuality=2
r.Shadow.MaxCSMResolution=2048
r.Shadow.DistanceScale=1.0
r.Shadow.CSM.MaxCascades=2
r.Shadow.RadiusThreshold=0.06
r.VolumetricFog=0
r.ContactShadows=0
r.DFShadowQuality=1
r.Mobile.SupportInsetShadows=True

[ShadowQuality@3]
r.ShadowQuality=2
r.Shadow.MaxCSMResolution=2048
r.Shadow.DistanceScale=1.0
r.Shadow.CSM.MaxCascades=3
r.Shadow.RadiusThreshold=0.06
r.VolumetricFog=0
r.ContactShadows=0
r.DFShadowQuality=2
r.Mobile.SupportInsetShadows=True

;-----------------------------------------------------------------------------------------------------------------

[ReflectionQuality@0]
r.SSR.Quality=0

[ReflectionQuality@1]
r.SSR.Quality=0

[ReflectionQuality@2]
r.SSR.Quality=0

[ReflectionQuality@3]
r.SSR.Quality=2

[ReflectionQuality@Cine]
r.SSR.Quality=2

; ------------------------------------------------------------------------------
; Postprocess settings
; ------------------------------------------------------------------------------

[PostProcessQuality@0]
r.Tonemapper.Quality=1
r.BloomQuality=0
r.DepthOfFieldQuality=0
r.LightShaftQuality=0
r.EyeAdaptationQuality=1
r.AmbientOcclusionLevels=0
r.LensFlareQuality=0

[PostProcessQuality@1]
r.Tonemapper.Quality=2
r.BloomQuality=1
r.DepthOfFieldQuality=0
r.LightShaftQuality=0
r.EyeAdaptationQuality=1
r.AmbientOcclusionLevels=0
r.LensFlareQuality=1

[PostProcessQuality@2]
r.Tonemapper.Quality=3
r.BloomQuality=2
r.DepthOfFieldQuality=0
r.LightShaftQuality=0
r.EyeAdaptationQuality=1
r.AmbientOcclusionLevels=1
r.LensFlareQuality=2

[PostProcessQuality@3]
r.Tonemapper.Quality=3
r.BloomQuality=3
r.DepthOfFieldQuality=1
r.LightShaftQuality=1
r.EyeAdaptationQuality=1
r.AmbientOcclusionLevels=1
r.LensFlareQuality=2

[AntiAliasingQuality@0]
r.Mobile.AntiAliasing=0
r.TemporalAA.Quality=0
r.FXAA.Quality=0

[AntiAliasingQuality@1]
r.Mobile.AntiAliasing=1
r.TemporalAA.Quality=0
r.FXAA.Quality=0

[AntiAliasingQuality@2]
r.Mobile.AntiAliasing=2
r.TemporalAA.Quality=0
r.FXAA.Quality=0

[AntiAliasingQuality@3]
r.Mobile.AntiAliasing=2
r.TemporalAA.Quality=0
r.FXAA.Quality=0

; ------------------------------------------------------------------------------
; Effects settings
; ------------------------------------------------------------------------------

[EffectsQuality@0]
r.SkyAtmosphere.AerialPerspectiveLUT.FastApplyOnOpaque=1 ; Always have FastSkyLUT 1 in this case to avoid wrong sky
r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice=1
r.SkyAtmosphere.AerialPerspectiveLUT.DepthResolution=4
r.SkyAtmosphere.FastSkyLUT=1
r.SkyAtmosphere.FastSkyLUT.SampleCountMin=1.0
r.SkyAtmosphere.FastSkyLUT.SampleCountMax=8.0
r.SkyAtmosphere.FastSkyLUT.Width=96
r.SkyAtmosphere.FastSkyLUT.Height=50
r.SkyAtmosphere.TransmittanceLUT.UseSmallFormat=1
r.RefractionQuality=0
fx.Niagara.QualityLevel=0

[EffectsQuality@1]
r.SkyAtmosphere.AerialPerspectiveLUT.FastApplyOnOpaque=1 ; Always have FastSkyLUT 1 in this case to avoid wrong sky
r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice=1
r.SkyAtmosphere.AerialPerspectiveLUT.DepthResolution=4
r.SkyAtmosphere.FastSkyLUT=1
r.SkyAtmosphere.FastSkyLUT.SampleCountMin=1.0
r.SkyAtmosphere.FastSkyLUT.SampleCountMax=8.0
r.SkyAtmosphere.FastSkyLUT.Width=96
r.SkyAtmosphere.FastSkyLUT.Height=50
r.SkyAtmosphere.TransmittanceLUT.UseSmallFormat=1
r.RefractionQuality=0
fx.Niagara.QualityLevel=0

[EffectsQuality@2]
r.SkyAtmosphere.AerialPerspectiveLUT.FastApplyOnOpaque=1 ; Always have FastSkyLUT 1 in this case to avoid wrong sky
r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice=1
r.SkyAtmosphere.AerialPerspectiveLUT.DepthResolution=4
r.SkyAtmosphere.FastSkyLUT=1
r.SkyAtmosphere.FastSkyLUT.SampleCountMin=1.0
r.SkyAtmosphere.FastSkyLUT.SampleCountMax=8.0
r.SkyAtmosphere.FastSkyLUT.Width=96
r.SkyAtmosphere.FastSkyLUT.Height=50
r.SkyAtmosphere.TransmittanceLUT.UseSmallFormat=1
r.RefractionQuality=1
fx.Niagara.QualityLevel=0

[EffectsQuality@3]
r.SkyAtmosphere.AerialPerspectiveLUT.FastApplyOnOpaque=1 ; Always have FastSkyLUT 1 in this case to avoid wrong sky
r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice=1
r.SkyAtmosphere.AerialPerspectiveLUT.DepthResolution=4
r.SkyAtmosphere.FastSkyLUT=1
r.SkyAtmosphere.FastSkyLUT.SampleCountMin=1.0
r.SkyAtmosphere.FastSkyLUT.SampleCountMax=8.0
r.SkyAtmosphere.FastSkyLUT.Width=96
r.SkyAtmosphere.FastSkyLUT.Height=50
r.SkyAtmosphere.TransmittanceLUT.UseSmallFormat=1
r.RefractionQuality=1
fx.Niagara.QualityLevel=1

; ------------------------------------------------------------------------------
; Texture settings
; ------------------------------------------------------------------------------

[TextureQuality@0]
r.MaxAnisotropy=1

[TextureQuality@1]
r.MaxAnisotropy=1

[TextureQuality@2]
r.MaxAnisotropy=2

[TextureQuality@3]
r.MaxAnisotropy=4

[ScalabilityPreview]
; don't want to preview any texture streaming changes in the editor
CVarDenyList=r.MaxAnisotropy
+CVarDenyList=r.Streaming.Boost
+CVarDenyList=r.Streaming.AmortizeCPUToGPUCopy
+CVarDenyList=r.Streaming.MaxNumTexturesToStreamPerFrame
+CVarDenyList=r.Streaming.PoolSize
