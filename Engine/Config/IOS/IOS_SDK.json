{
	"ParentSDKFile": "../Apple/Apple_SDK.json",

	"MinSoftwareVersion":  "15.0"
}
